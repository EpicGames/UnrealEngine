// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedPlatformGroups("Desktop")]
public class DevHttp : ModuleRules
{
	public DevHttp(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] { "Core", "SSL", "HTTP" });
		
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows) && Target.WindowsPlatform.bUseXCurl)
		{
			PublicDependencyModuleNames.Add("XCurl");
		}
		else
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "libcurl");
		}
		AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");

		PrivateDefinitions.Add("WITH_CURL=1");
	}
}
