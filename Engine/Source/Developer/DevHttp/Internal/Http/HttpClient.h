// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Memory/MemoryFwd.h"
#include "Templates/Function.h"
#include "Templates/PimplPtr.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

namespace UE { class IHttpClient; }
namespace UE { class IHttpConnectionPool; }
namespace UE { class IHttpReceiver; }
namespace UE { class IHttpRequest; }
namespace UE { class IHttpResponse; }
namespace UE { struct FHttpClientParams; }
namespace UE { struct FHttpConnectionPoolParams; }
namespace UE { struct FHttpRequestParams; }
namespace UE { struct FHttpResponseStats; }
namespace UE::Http::Private { struct FHttpRequestQueueData; }

namespace UE::Http::Private
{

template <typename T>
struct THttpDestroyer
{
	void operator()(void* Object) const
	{
		if (Object)
		{
			((T*)Object)->Destroy();
		}
	}
};

} // UE::Http::Private

namespace UE
{

template <typename Type, typename BaseType = Type>
using THttpUniquePtr = TUniquePtr<Type, Http::Private::THttpDestroyer<BaseType>>;

enum class EHttpMethod : uint8
{
	Get,
	Put,
	Post,
	Head,
	Delete,
};

DEVHTTP_API FAnsiStringView LexToString(EHttpMethod Method);
DEVHTTP_API bool TryLexFromString(EHttpMethod& OutMethod, FAnsiStringView View);

enum class EHttpMediaType : uint8
{
	Any,
	Binary,
	Text,
	Json,
	Yaml,
	CbObject,
	CbPackage,
	CbPackageOffer,
	CompressedBinary,
	FormUrlEncoded,
};

DEVHTTP_API FAnsiStringView LexToString(EHttpMediaType MediaType);
DEVHTTP_API bool TryLexFromString(EHttpMediaType& OutMediaType, FAnsiStringView View);

enum class EHttpVersion : uint8
{
	/** Determine version automatically. */
	None = 0,
	/** Use HTTP/1.0 */
	V1_0,
	/** Use HTTP/1.1 */
	V1_1,
	/** Try HTTP/2 with fallback to HTTP/1.1 */
	V2,
	/** Use HTTP/2 */
	V2Only,
};

enum class EHttpTlsLevel : uint8
{
	/** Do not use TLS. */
	None = 0,
	/** Try to use TLS and fall back to none. */
	Try,
	/** Require TLS. */
	All,
};

enum class EHttpErrorCode : uint8
{
	/** Success. */
	None = 0,
	/** Unknown. Check IHttpResponse::GetError(). */
	Unknown,
	/** Request was canceled. */
	Canceled,
	/** Failed to resolve the host. */
	ResolveHost,
	/** Failed to establish a connection. */
	Connect,
	/** Failed to establish a TLS connection. */
	TlsConnect,
	/** Failed to verify the certificate of the peer. */
	TlsPeerVerification,
	/** Failed because the timeout period was exceeded. */
	TimedOut,
};

DEVHTTP_API FAnsiStringView LexToString(EHttpErrorCode ErrorCode);
DEVHTTP_API bool TryLexFromString(EHttpErrorCode& OutErrorCode, FAnsiStringView View);

class IHttpManager
{
public:
	static DEVHTTP_API IHttpManager& Get();

	[[nodiscard]] virtual THttpUniquePtr<IHttpConnectionPool> CreateConnectionPool(const FHttpConnectionPoolParams& Params) = 0;
};

class IHttpConnectionPool
{
public:
	[[nodiscard]] virtual THttpUniquePtr<IHttpClient> CreateClient(const FHttpClientParams& Params) = 0;

protected:
	friend Http::Private::THttpDestroyer<IHttpConnectionPool>;
	virtual ~IHttpConnectionPool() = default;
	virtual void Destroy() = 0;
};

struct FHttpConnectionPoolParams final
{
	/** Maximum number of concurrent connections created by the pool. Use 0 for the default limit. */
	uint32 MaxConnections = 0;

	/** Minimum number of concurrent connections maintained by the pool for reuse. Use 0 for the default limit. */
	uint32 MinConnections = 0;

	/** Maximum number of concurrent requests in flight for a connection when multiplexing. Use 0 for the default limit. */
	uint32 MaxRequestsPerConnection = 0;

	/** Allow requests to send async using the pool. Requests will block on execution when this is disabled. */
	bool bAllowAsync = true;
};

class IHttpClient
{
public:
	/**
	 * Try to create a request.
	 *
	 * The request is counted against the maximum request count as soon as it is created.
	 *
	 * Returns null when the maximum request count has been reached. Retry after destroying a request.
	 */
	[[nodiscard]] virtual THttpUniquePtr<IHttpRequest> TryCreateRequest(const FHttpRequestParams& Params) = 0;

protected:
	friend Http::Private::THttpDestroyer<IHttpClient>;
	virtual ~IHttpClient() = default;
	virtual void Destroy() = 0;
};

struct FHttpClientParams final
{
	/** Invoked whenever a request from this client is destroyed. This may be used to retry creating a request. */
	TFunction<void()> OnDestroyRequest;

	/** Maximum number of concurrent requests created by the client. Use 0 for the default limit. */
	uint32 MaxRequests = 0;

	/** Minimum number of concurrent requests maintained by the client for reuse. Use 0 for the default limit. */
	uint32 MinRequests = 0;

	/** Domain name cache timeout in seconds. Use 0 to disable the cache. */
	uint32 DnsCacheTimeout = 60;

	/** Connection timeout in milliseconds. Use 0 for the default timeout. */
	uint32 ConnectTimeoutMs = 0;

	/** Total timeout in milliseconds. Use 0 for no timeout. */
	uint32 TotalTimeoutMs = 0;

	/** Average transfer speed in bytes/s below which a request will abort. Use 0 to disable. */
	uint32 LowSpeedLimit = 0;

	/** Time in seconds after which a request will abort if it stays below LowSpeedLimit. Use 0 to disable. */
	uint32 LowSpeedTime = 0;

	/** HTTP version to use for requests created by the client. */
	EHttpVersion Version = EHttpVersion::None;

	/** Level of TLS to use for requests created by the client. */
	EHttpTlsLevel TlsLevel = EHttpTlsLevel::None;

	/** Follow redirects in responses automatically. */
	bool bFollowRedirects : 1;
	/** Follow redirects of the corresponding status code without rewriting POST to GET. */
	bool bFollow301Post : 1;
	bool bFollow302Post : 1;
	bool bFollow303Post : 1;

	/** Avoid use of configured proxy on the client. */
	bool bBypassProxy : 1;

	/** Verbose logging for requests created by the client. */
	bool bVerbose : 1;

	inline FHttpClientParams()
		: bFollowRedirects(false)
		, bFollow301Post(false)
		, bFollow302Post(false)
		, bFollow303Post(false)
		, bBypassProxy(false)
		, bVerbose(false)
	{
	}
};

class IHttpRequest
{
public:
	/**
	 * Reset the request to an empty state equivalent to when it was created.
	 *
	 * The request must be idle to be reset, which means any associated response must have completed.
	 */
	virtual void Reset() = 0;

	/** Set the URI used for the request, including the scheme and authority. */
	virtual void SetUri(FAnsiStringView Uri) = 0;

	/** Set a Unix socket path that should be used for sending requests */
	virtual void SetUnixSocketPath(FAnsiStringView SocketPath) = 0;

	/** Set the method used for the request. Defaults to GET. */
	virtual void SetMethod(EHttpMethod Method) = 0;

	/**
	 * Set the body for PUT and POST requests.
	 *
	 * Body must be owned or otherwise remain valid until the request is complete.
	 */
	virtual void SetBody(const FCompositeBuffer& Body) = 0;

	/** Set the Content-Type header with a known media type. Use AddHeader() for other types. */
	DEVHTTP_API void SetContentType(EHttpMediaType Type, FAnsiStringView Param = {});

	/** Add an Accept header with a known media type. Use AddHeader() for other types. */
	DEVHTTP_API void AddAcceptType(EHttpMediaType Type, float Weight = 1.0);

	/** Add the header. An empty value removes the header. */
	virtual void AddHeader(FAnsiStringView Name, FAnsiStringView Value) = 0;

	/**
	 * Send the request on the client that it was created from, and wait for it to complete before returning.
	 *
	 * Request must be idle to send, which means any previous response must be complete.
	 *
	 * Receiver must remain valid until OnComplete is called or until it returns another receiver
	 * to be used for subsequent events.
	 */
	virtual void Send(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse) = 0;

	/**
	 * Asynchronously send the request on the client that it was created from.
	 *
	 * Request must be idle to send, which means any previous response must be complete.
	 *
	 * Receiver must remain valid until OnComplete is called or until it returns another receiver
	 * to be used for subsequent events.
	 *
	 * Response must be kept alive until OnComplete is called. Cancel to complete immediately.
	 */
	virtual void SendAsync(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse) = 0;

protected:
	friend Http::Private::THttpDestroyer<IHttpRequest>;
	virtual ~IHttpRequest() = default;
	virtual void Destroy() = 0;
};

struct FHttpRequestParams final
{
	/** Whether to force creation of the request even if it would exceed the configured maximum. */
	bool bIgnoreMaxRequests = false;
};

class IHttpResponseMonitor
{
public:
	/** Cancels the request associated with this response. */
	virtual void Cancel() = 0;

	/** Waits for the request associated with this response to be complete. */
	virtual void Wait() const = 0;

	/** Returns true if the request associated with this response is complete, and false otherwise. */
	[[nodiscard]] virtual bool Poll() const = 0;

	virtual void AddRef() const = 0;
	virtual void Release() const = 0;
};

class IHttpResponse
{
public:
	/** Returns a monitor for the response that can poll, wait, or cancel it. */
	[[nodiscard]] virtual TRefCountPtr<IHttpResponseMonitor> GetMonitor() = 0;

	/** Cancels the request associated with this response. */
	virtual void Cancel() = 0;

	/** Waits for the request associated with this response to be complete. */
	virtual void Wait() const = 0;

	/** Returns true if the request associated with this response is complete, and false otherwise. */
	[[nodiscard]] virtual bool Poll() const = 0;

	/** Get the URI used for the request associated with this response. Always available. */
	[[nodiscard]] virtual FAnsiStringView GetUri() const = 0;

	/** Get the method used for the request associated with this response. Always available. */
	[[nodiscard]] virtual EHttpMethod GetMethod() const = 0;

	/**
	 * Returns the status code of the response. Available once the response status has been received.
	 *
	 * The return value is negative until the response status has been received.
	 * The return value is 0 if an error occurred. See GetError().
	 */
	[[nodiscard]] virtual int32 GetStatusCode() const = 0;

	/** Returns the error code from the HTTP stack. Available when the response is complete. */
	[[nodiscard]] virtual EHttpErrorCode GetErrorCode() const = 0;

	/** Returns the optional error message from the HTTP stack. Available when the status code is 0. */
	[[nodiscard]] virtual FAnsiStringView GetError() const = 0;

	/** Returns information about the request such as size, rate, and duration. Available when complete. */
	[[nodiscard]] virtual const FHttpResponseStats& GetStats() const = 0;

	/** Returns every header in the response. Available once OnHeaders() has been invoked. */
	[[nodiscard]] virtual TConstArrayView<FAnsiStringView> GetAllHeaders() const = 0;

	/** Returns the value of the first header matching the name, or empty if there was no matching header. */
	[[nodiscard]] DEVHTTP_API FAnsiStringView GetHeader(FAnsiStringView Name) const;

	/**
	 * Fills the array of header values and returns the total number of headers matching the name.
	 *
	 * The return value can be greater than the array size if there are more matching headers than the array can hold.
	 */
	[[nodiscard]] DEVHTTP_API int32 GetHeaders(FAnsiStringView Name, TArrayView<FAnsiStringView> OutValues) const;

	/** Returns the content type of the response, or Any if there was no matching header. */
	[[nodiscard]] DEVHTTP_API EHttpMediaType GetContentType() const;

	/** Returns the content length of the response, or MAX_uint64 if there was no matching header. */
	[[nodiscard]] DEVHTTP_API uint64 GetContentLength() const;

protected:
	friend Http::Private::THttpDestroyer<IHttpResponse>;
	virtual ~IHttpResponse() = default;

	/** Cancel() if not yet invoking OnComplete(), and then Release(). */
	virtual void Destroy() = 0;
};

/** Appends the URI, method, status code, and optional error from the response to the builder. */
DEVHTTP_API FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const IHttpResponse& Response);

struct FHttpResponseStats final
{
	/** Total size of data sent to the server, in bytes. */
	uint64 SendSize = 0;
	/** Total size of data received from the server, in bytes. */
	uint64 RecvSize = 0;
	/** Average rate at which data was sent to the server, in bytes/s. */
	uint64 SendRate = 0;
	/** Average rate at which data was received from the server, in bytes/s. */
	uint64 RecvRate = 0;

	/** Time from the start of the request until the name was resolved, in seconds. */
	double NameResolveTime = 0.0;
	/** Time from the start of the request until the connection was established, in seconds. */
	double ConnectTime = 0.0;
	/** Time from the start of the request until the TLS connection was complete, in seconds. */
	double TlsConnectTime = 0.0;
	/** Time from the start of the request until the request was sent but before the first byte was received, in seconds. */
	double PreTransferTime = 0.0;
	/** Time from the start of the request until the first byte was received, in seconds. */
	double StartTransferTime = 0.0;
	/** Time from the start of the request until response was complete, in seconds. */
	double TotalTime = 0.0;

	/** Time from when the request was sent until the first byte was received, in seconds. If no byte has been received, latency is 0. */
	double GetLatency() const
	{
		return FMath::Max(0.0f, StartTransferTime - PreTransferTime);
	}
};

/**
 * Interface to receive the headers and body of the response.
 *
 * Functions on this interface must be implemented to do the minimal work possible, because their
 * execution can block other requests from being processed.
 */
class IHttpReceiver
{
public:
	virtual ~IHttpReceiver() = default;

	/**
	 * Invoked when the response has been created, before it begins executing.
	 *
	 * Returns the receiver to use for the next part of the response and can return itself.
	 */
	virtual IHttpReceiver* OnCreate(IHttpResponse& Response) { return this; }

	/**
	 * Invoked when every header has been received for the response.
	 *
	 * Returns the receiver to use for the next part of the response and can return itself.
	 */
	virtual IHttpReceiver* OnHeaders(IHttpResponse& Response) { return this; }

	/**
	 * Invoked when a part of the body has been received.
	 *
	 * The Data parameter can be reassigned to point to the tail of the view that is not consumed
	 * by this receiver, and will be passed to the receiver that is returned.
	 *
	 * Returns the receiver to use for the next part of the response and can return itself.
	 */
	virtual IHttpReceiver* OnBody(IHttpResponse& Response, FMemoryView& Data) { return this; }

	/**
	 * Invoked when the response is complete. Always invoked exactly once for each response.
	 *
	 * Check the status code to detect errors. Can retry using the same request.
	 *
	 * It is valid for implementations of this function to delete the response.
	 *
	 * Returns the next receiver to handle OnComplete, or null if there are no others.
	 */
	virtual IHttpReceiver* OnComplete(IHttpResponse& Response) { return this; }
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FHttpByteArrayReceiver final : public IHttpReceiver
{
public:
	explicit DEVHTTP_API FHttpByteArrayReceiver(TArray64<uint8>& OutArray, IHttpReceiver* Next = nullptr);

	DEVHTTP_API IHttpReceiver* OnBody(IHttpResponse& Response, FMemoryView& Data) final;
	IHttpReceiver* OnComplete(IHttpResponse& Response) final { return Next; }

private:
	TArray64<uint8>& Array;
	IHttpReceiver* Next;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Manages a client by letting callers queue to receive a request when one is not immediately available. */
class FHttpRequestQueue final
{
public:
	/** Create a null request queue. Asserts if CreateRequest is called. */
	FHttpRequestQueue() = default;

	DEVHTTP_API FHttpRequestQueue(IHttpConnectionPool& ConnectionPool, const FHttpClientParams& ClientParams);

	/** Blocks until a request can be created on the underlying client. */
	[[nodiscard]] DEVHTTP_API THttpUniquePtr<IHttpRequest> CreateRequest(const FHttpRequestParams& Params);

private:
	TPimplPtr<Http::Private::FHttpRequestQueueData> Data;
};

} // UE
