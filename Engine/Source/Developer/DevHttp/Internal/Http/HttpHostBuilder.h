// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"

namespace UE
{

class FHttpHostBuilder
{
public:
	// accept a semicolon seperated list of hosts to pick from
	DEVHTTP_API void AddFromString(FAnsiStringView HostList);
	DEVHTTP_API void AddFromString(FStringView HostList);

	// a url that is expected to host a /status/peers endpoint with endpoints we should attempt to use
	DEVHTTP_API void AddFromEndpoint(FAnsiStringView HostUrl, FAnsiStringView AccessToken);

	// produces a string of all the host candidates used for display purposes
	DEVHTTP_API FString GetHostCandidatesString() const;

	DEVHTTP_API bool ResolveHost(double WarningTimeoutSeconds, double TimeoutSeconds, FAnsiStringBuilderBase& OutHost, double& OutLatency);
private:
	static bool BenchmarkHostList(TConstArrayView<FString> HostCandidates, double WarningTimeoutSeconds, double TimeoutSeconds, FAnsiStringBuilderBase& OutHost, double& OutLatency);

	TArray<FString> HostCandidates;
};

} // UE
