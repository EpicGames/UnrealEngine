// Copyright Epic Games, Inc. All Rights Reserved.

#include "Http/HttpClient.h"

#include "Async/InheritedContext.h"
#include "Async/ManualResetEvent.h"
#include "Containers/AnsiString.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Containers/LockFreeList.h"
#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Thread.h"
#include "Logging/StructuredLog.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/MemoryView.h"
#include "Misc/App.h"
#include "Misc/EngineVersion.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinary.h"
#include "String/Find.h"
#include "Templates/RefCounting.h"
#include "Templates/UnrealTemplate.h"

#if WITH_SSL
#include "Ssl.h"
#include <openssl/ssl.h>
#endif

#ifndef CURL_NO_OLDIES
#define CURL_NO_OLDIES
#endif

#if PLATFORM_MICROSOFT
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#endif
#ifdef PLATFORM_CURL_INCLUDE
#include PLATFORM_CURL_INCLUDE
#else
#include "curl/curl.h"
#endif
#if PLATFORM_MICROSOFT
#include "Microsoft/HideMicrosoftPlatformTypes.h"
#endif
#include "ThirdParty/UECurl.h"

#include <atomic>

namespace UE { class FCurlHttpClient; }
namespace UE { class FCurlHttpRequest; }
namespace UE { class FCurlHttpResponse; }

namespace UE::CurlHttp::Private
{

template <typename T>
static bool TryIncrement(std::atomic<T>& Value, const T Max)
{
	for (T Existing = Value.load(std::memory_order_relaxed);;)
	{
		if (Existing >= Max)
		{
			return false;
		}
		if (Value.compare_exchange_weak(Existing, Existing + 1, std::memory_order_relaxed))
		{
			return true;
		}
	}
}

template <typename T>
static T AtomicFetchOr(std::atomic<T>& Atomic, const T Value, std::memory_order MemoryOrder = std::memory_order_seq_cst)
{
	for (T Existing = Atomic.load(MemoryOrder);;)
	{
		if (Atomic.compare_exchange_weak(Existing, Existing | Value, MemoryOrder))
		{
			return Existing;
		}
	}
}

template <typename T, typename PredicateType>
static bool AtomicFetchOrIf(
	std::atomic<T>& Atomic,
	const T Value,
	PredicateType Predicate,
	std::memory_order MemoryOrder = std::memory_order_seq_cst)
{
	for (T Existing = Atomic.load(MemoryOrder);;)
	{
		if (!Predicate(Existing))
		{
			return false;
		}
		if (Atomic.compare_exchange_weak(Existing, Existing | Value, MemoryOrder))
		{
			return true;
		}
	}
}

} // UE::CurlHttp::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::Curl
{

static FCurlInitializeData InitializeData;

}

namespace UE
{

DEFINE_LOG_CATEGORY_STATIC(LogHttp, Display, All);

class FCurlHttpHeaders
{
public:
	FCurlHttpHeaders() = default;
	FCurlHttpHeaders(const FCurlHttpHeaders&) = delete;
	FCurlHttpHeaders& operator=(const FCurlHttpHeaders&) = delete;

	inline ~FCurlHttpHeaders()
	{
		curl_slist_free_all(List);
	}

	inline void Reset()
	{
		curl_slist_free_all(List);
		List = nullptr;
	}

	inline void AddHeader(const ANSICHAR* Header)
	{
		List = curl_slist_append(List, Header);
	}

	inline curl_slist* GetList() const { return List; }

private:
	curl_slist* List = nullptr;
};

class FCurlHttpManager final : public IHttpManager
{
public:
	FCurlHttpManager();

	THttpUniquePtr<IHttpConnectionPool> CreateConnectionPool(const FHttpConnectionPoolParams& Params) final;

	void SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers);

private:
	const FCbObjectId SessionId = FApp::GetSessionObjectId();
	std::atomic<uint32> RequestId = 1;
	FUtf8String UserAgent;
};

class FCurlHttpConnectionPool final : public IHttpConnectionPool
{
public:
	FCurlHttpConnectionPool(FCurlHttpManager& Manager, const FHttpConnectionPoolParams& Params);

	THttpUniquePtr<IHttpClient> CreateClient(const FHttpClientParams& Params) final;
	void DeleteClient(FCurlHttpClient* Client);

	void SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers) const;

	bool BeginAsyncRequest(FCurlHttpResponse* Response);
	void CancelAsyncRequest(FCurlHttpResponse* Response);

private:
	~FCurlHttpConnectionPool() final;
	void Destroy() final { delete this; }

	void ThreadLoop();
	void CompleteRequest(CURL* Curl, CURLcode* OptionalReturnCode);

	static void CurlLock(CURL* Curl, curl_lock_data Data, curl_lock_access Access, void* Param);
	static void CurlUnlock(CURL* Curl, curl_lock_data Data, void* Param);

	static void AssertShareCodeOk(CURLSHcode Code);
	static void AssertMultiCodeOk(CURLMcode Code);

	FCurlHttpManager& Manager;
	CURLSH* CurlShare;
	CURLM* CurlMulti;

	FRWLock Locks[CURL_LOCK_DATA_LAST];
	bool WriteLocked[CURL_LOCK_DATA_LAST]{};

	std::atomic<uint32> ClientCount = 0;

	enum class EThreadCommandType
	{
		Begin,
		Cancel,
	};

	struct FThreadCommand
	{
		TRefCountPtr<FCurlHttpResponse> Response;
		EThreadCommandType Type;
	};

	TConsumeAllMpmcQueue<FThreadCommand> ThreadCommands;
	FThread Thread;
	std::atomic<bool> bThreadStarting;
	std::atomic<bool> bThreadStopping;
};

class FCurlHttpClient final : public IHttpClient
{
public:
	FCurlHttpClient(FCurlHttpConnectionPool& ConnectionPool, const FHttpClientParams& Params);

	THttpUniquePtr<IHttpRequest> TryCreateRequest(const FHttpRequestParams& Params) final;
	void DeleteRequest(FCurlHttpRequest* Request);

	void SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers) const;

	bool BeginAsyncRequest(FCurlHttpResponse* Response);
	void CancelAsyncRequest(FCurlHttpResponse* Response);

private:
	~FCurlHttpClient() final;
	void Destroy() final { delete this; }

	static long ConvertVersion(EHttpVersion Version);
	static long ConvertTlsLevel(EHttpTlsLevel Level);

	FCurlHttpConnectionPool& ConnectionPool;
	TLockFreePointerListLIFO<FCurlHttpRequest> RequestPool;
	std::atomic<uint32> RequestPoolCount = 0;
	std::atomic<uint32> RequestCount = 0;
	FHttpClientParams Params;
};

class FCurlHttpRequest final : public IHttpRequest
{
public:
	explicit FCurlHttpRequest(FCurlHttpClient& Client);
	~FCurlHttpRequest() final;

	void Reset() final;
	void SetUri(FAnsiStringView Uri) final;
	void SetUnixSocketPath(FAnsiStringView Uri) final;
	void SetMethod(EHttpMethod Method) final;
	void SetBody(const FCompositeBuffer& Body) final;
	void AddHeader(FAnsiStringView Name, FAnsiStringView Value) final;

	void Send(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse) final;
	void SendAsync(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse) final;

	void OnComplete(CURLcode Code);

private:
	void Destroy() final { Client.DeleteRequest(this); }

	void SetDefaultOptions();
	void CheckIdle(const TCHAR* FunctionName);

	FCurlHttpResponse* CreateResponse(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse);

	static size_t CurlRead(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param);
	static size_t CurlSeek(void* Param, curl_off_t Offset, int Origin);
#if WITH_SSL
	static CURLcode CurlSslContext(CURL* Curl, void* Context, void* Param);
	static int SslCertVerify(int PreverifyOk, X509_STORE_CTX* Context);
#endif

	EHttpMethod Method;
	FAnsiString Uri;
	FAnsiString UnixSocketPath;
	FCurlHttpClient& Client;
	FCurlHttpHeaders Headers;
	FCompositeBuffer Body;
	uint64 BodyOffset = 0;
	CURL* Curl = nullptr;
	std::atomic<FCurlHttpResponse*> Response = nullptr;
};

enum class ECurlHttpResponseState : uint8
{
	None = 0,
	Complete = 1 << 0,
	Canceled = 1 << 1,
};

ENUM_CLASS_FLAGS(ECurlHttpResponseState);

class FCurlHttpResponse final : public IHttpResponse, public IHttpResponseMonitor, private FInheritedContextBase
{
public:
	FCurlHttpResponse(CURL* Curl, EHttpMethod Method, FAnsiStringView Uri, IHttpReceiver* Receiver);

	bool Create();

	CURL* GetCurl() { return Curl; }
	void SetClient(FCurlHttpClient* InClient) { Client = InClient; }
	void SetComplete(CURLcode Code);

	TRefCountPtr<IHttpResponseMonitor> GetMonitor() final { return this; }

	void Cancel() final;
	void Wait() const final { CompleteEvent.Wait(); }
	bool Poll() const final { return EnumHasAnyFlags(State.load(), ECurlHttpResponseState::Complete); }
	bool IsCanceled() const { return EnumHasAnyFlags(State.load(std::memory_order_relaxed), ECurlHttpResponseState::Canceled); }

	FAnsiStringView GetUri() const final { return Uri; }
	EHttpMethod GetMethod() const final { return Method; }
	int32 GetStatusCode() const final;
	EHttpErrorCode GetErrorCode() const final { return ErrorCode; }
	FAnsiStringView GetError() const final { return Error; }
	const FHttpResponseStats& GetStats() const final;
	TConstArrayView<FAnsiStringView> GetAllHeaders() const final { return HeaderViews; }

	void AddRef() const final;
	void Release() const final;

private:
	~FCurlHttpResponse() final;
	void Destroy() final;

	void ConditionallySetResponseStatus();
	bool WriteHeader(FAnsiStringView Header);
	bool WriteBody(FMemoryView Body);

	static size_t CurlDebug(CURL* Curl, curl_infotype DebugInfoType, char* DebugInfo, size_t DebugInfoSize, void* Param);
	static size_t CurlHeader(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param);
	static size_t CurlWrite(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param);

	static EHttpErrorCode ConvertErrorCode(CURLcode Code);

	FAnsiString Uri;
	EHttpMethod Method;
	bool bHasBody = false;
	EHttpErrorCode ErrorCode = EHttpErrorCode::Unknown;
	std::atomic<ECurlHttpResponseState> State = ECurlHttpResponseState::None;
	std::atomic<int32> StatusCode = -1;
	mutable std::atomic<uint32> ReferenceCount = 0;

	CURL* Curl; // null when complete
	IHttpReceiver* Receiver;
	FCurlHttpClient* Client = nullptr;
	TArray<FAnsiStringView, TInlineAllocator<32>> HeaderViews;
	TArray<int32, TInlineAllocator<32>> HeaderLengths;
	TAnsiStringBuilder<4096> Headers;
	FAnsiStringView Error;
	ANSICHAR ErrorBuffer[CURL_ERROR_SIZE]{};
	FHttpResponseStats Stats;
	mutable FManualResetEvent CompleteEvent;

	enum class EHttpReceiverFunction : uint8
	{
		None = 0,
		OnCreate,
		OnHeaders,
		OnBody,
		OnComplete,
	};
	inline static thread_local EHttpReceiverFunction ReceiverFunction;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCurlHttpManager::FCurlHttpManager()
{
	// User-Agent: UnrealEngine/X.Y.Z-<CL> (<Platform>; <Config> <TargetType>; <BranchName>) <AppName>[CommandletName] (<ProjectName>)
	TUtf8StringBuilder<256> UserAgentBuilder;
	const FEngineVersion& Version = FEngineVersion::Current();
	UserAgentBuilder << ANSITEXTVIEW("User-Agent: UnrealEngine/")
		<< Version.GetMajor() << '.' << Version.GetMinor() << '.' << Version.GetPatch() << '-' << Version.GetChangelist()
		<< ANSITEXTVIEW(" (") << FPlatformProperties::PlatformName()
		<< ANSITEXTVIEW("; ") << LexToString(FApp::GetBuildConfiguration()) << ' ' << LexToString(FApp::GetBuildTargetType())
		<< ANSITEXTVIEW("; ") << FApp::GetBranchName()
		<< ANSITEXTVIEW(") ") << FApp::GetName();

	if (IsRunningCommandlet())
	{
		FString CommandletName;
		FParse::Value(FCommandLine::Get(), TEXT("Run="), CommandletName);
		CommandletName.ToLowerInline();
		UserAgentBuilder << '[' << CommandletName << ']';
	}

	if (FApp::HasProjectName() && FApp::GetName() != FApp::GetProjectName())
	{
		UserAgentBuilder << ANSITEXTVIEW(" (") << FApp::GetProjectName() << ')';
	}
	UserAgent = UserAgentBuilder;
}

THttpUniquePtr<IHttpConnectionPool> FCurlHttpManager::CreateConnectionPool(const FHttpConnectionPoolParams& Params)
{
	return THttpUniquePtr<IHttpConnectionPool>(new FCurlHttpConnectionPool(*this, Params));
}

void FCurlHttpManager::SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers)
{
	curl_easy_setopt(Curl, CURLOPT_NOSIGNAL, 1L);
	curl_easy_setopt(Curl, CURLOPT_USERAGENT, *UserAgent);

	Headers.AddHeader(*WriteToAnsiString<64>(ANSITEXTVIEW("UE-IsBuildMachine: "), GIsBuildMachine));
	Headers.AddHeader(*WriteToAnsiString<64>(ANSITEXTVIEW("UE-Session: "), SessionId));
	Headers.AddHeader(*WriteToAnsiString<32>(ANSITEXTVIEW("UE-Request: "), RequestId.fetch_add(1, std::memory_order_relaxed)));

	// Remove the Expect: 100-Continue header that curl adds by default because it adds latency.
	Headers.AddHeader("Expect:");
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCurlHttpConnectionPool::FCurlHttpConnectionPool(FCurlHttpManager& InManager, const FHttpConnectionPoolParams& Params)
	: Manager(InManager)
	, CurlShare()
	, CurlMulti()
	, bThreadStarting(false)
	, bThreadStopping(!Params.bAllowAsync || !FPlatformProcess::SupportsMultithreading())
{
	CURLcode InitResult = (CURLcode)UE::Curl::ConditionalInitialize(UE::Curl::InitializeData);
	if (InitResult != CURLE_OK)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpConnectionPool could not initialize libcurl (result=%d).", (int32)InitResult);
	}

	CurlShare = curl_share_init();
	CurlMulti = curl_multi_init();
	curl_share_setopt(CurlShare, CURLSHOPT_USERDATA, this);
	curl_share_setopt(CurlShare, CURLSHOPT_LOCKFUNC, CurlLock);
	curl_share_setopt(CurlShare, CURLSHOPT_UNLOCKFUNC, CurlUnlock);
	curl_share_setopt(CurlShare, CURLSHOPT_SHARE, CURL_LOCK_DATA_DNS);
	curl_share_setopt(CurlShare, CURLSHOPT_SHARE, CURL_LOCK_DATA_SSL_SESSION);

	checkf(!Params.MaxConnections || !Params.MinConnections || Params.MinConnections <= Params.MaxConnections,
		TEXT("MinConnections (%u) exceeds MaxConnections (%u)"), Params.MinConnections, Params.MaxConnections);
	constexpr uint32 DefaultConnectionCount = 8;
	const uint32 MaxConnections = Params.MaxConnections ? Params.MaxConnections : FMath::Max(Params.MinConnections, DefaultConnectionCount);
	const uint32 MinConnections = Params.MinConnections ? Params.MinConnections : FMath::Min(Params.MaxConnections, DefaultConnectionCount);
	curl_multi_setopt(CurlMulti, CURLMOPT_MAXCONNECTS, MinConnections);
	curl_multi_setopt(CurlMulti, CURLMOPT_MAX_TOTAL_CONNECTIONS, MaxConnections);
	if (Params.MaxRequestsPerConnection)
	{
		curl_multi_setopt(CurlMulti, CURLMOPT_MAX_CONCURRENT_STREAMS, (long)FMath::Min(Params.MaxRequestsPerConnection, (uint32)MAX_int32));
	}
	curl_multi_setopt(CurlMulti, CURLMOPT_PIPELINING, CURLPIPE_MULTIPLEX);
}

FCurlHttpConnectionPool::~FCurlHttpConnectionPool()
{
	checkf(ClientCount.load(std::memory_order_relaxed) == 0,
		TEXT("Connection pool has %u clients that were not deleted."), ClientCount.load());

	bThreadStopping.store(true, std::memory_order_relaxed);
	AssertMultiCodeOk(curl_multi_wakeup(CurlMulti));
	if (Thread.IsJoinable())
	{
		Thread.Join();
	}

	check(ThreadCommands.IsEmpty());

	AssertMultiCodeOk(curl_multi_cleanup(CurlMulti));
	AssertShareCodeOk(curl_share_cleanup(CurlShare));
}

THttpUniquePtr<IHttpClient> FCurlHttpConnectionPool::CreateClient(const FHttpClientParams& Params)
{
	ClientCount.fetch_add(1, std::memory_order_relaxed);
	return THttpUniquePtr<IHttpClient>(new FCurlHttpClient(*this, Params));
}

void FCurlHttpConnectionPool::DeleteClient(FCurlHttpClient* Client)
{
	ClientCount.fetch_sub(1, std::memory_order_relaxed);
}

void FCurlHttpConnectionPool::SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers) const
{
	Manager.SetDefaultOptions(Curl, Headers);

	curl_easy_setopt(Curl, CURLOPT_SHARE, CurlShare);
}

bool FCurlHttpConnectionPool::BeginAsyncRequest(FCurlHttpResponse* Response)
{
	if (bThreadStopping.load(std::memory_order_relaxed))
	{
		return false;
	}
	if (ThreadCommands.ProduceItem(FThreadCommand{Response, EThreadCommandType::Begin}) == EConsumeAllMpmcQueueResult::WasEmpty)
	{
		AssertMultiCodeOk(curl_multi_wakeup(CurlMulti));
	}
	if (!bThreadStarting.load(std::memory_order_relaxed) && !bThreadStarting.exchange(true, std::memory_order_relaxed))
	{
		Thread = FThread(TEXT("HttpConnectionPool"), [this] { ThreadLoop(); }, 128 * 1024);
	}
	return true;
}

void FCurlHttpConnectionPool::CancelAsyncRequest(FCurlHttpResponse* Response)
{
	if (ThreadCommands.ProduceItem(FThreadCommand{Response, EThreadCommandType::Cancel}) == EConsumeAllMpmcQueueResult::WasEmpty)
	{
		AssertMultiCodeOk(curl_multi_wakeup(CurlMulti));
	}
}

void FCurlHttpConnectionPool::ThreadLoop()
{
	while (!ThreadCommands.IsEmpty() || !bThreadStopping.load(std::memory_order_relaxed))
	{
		ThreadCommands.ConsumeAllFifo([this](FThreadCommand Command)
		{
			if (CURL* Curl = Command.Response->GetCurl())
			{
				switch (Command.Type)
				{
				case EThreadCommandType::Begin:
					AssertMultiCodeOk(curl_multi_add_handle(CurlMulti, Curl));
					break;
				case EThreadCommandType::Cancel:
					CompleteRequest(Curl, nullptr);
					break;
				}
			}
		});

		int ActiveRequests = 0;
		AssertMultiCodeOk(curl_multi_perform(CurlMulti, &ActiveRequests));

		for (int MessagesInQueue = 0; CURLMsg* Message = curl_multi_info_read(CurlMulti, &MessagesInQueue);)
		{
			if (Message->msg == CURLMSG_DONE)
			{
				CompleteRequest(Message->easy_handle, &Message->data.result);
			}
		}

		constexpr int WaitTimeMs = 60'000;
		AssertMultiCodeOk(curl_multi_poll(CurlMulti, nullptr, 0, WaitTimeMs, nullptr));
	}
}

void FCurlHttpConnectionPool::CompleteRequest(CURL* Curl, CURLcode* OptionalReturnCode)
{
	AssertMultiCodeOk(curl_multi_remove_handle(CurlMulti, Curl));
	void* Request = nullptr;
	curl_easy_getinfo(Curl, CURLINFO_PRIVATE, &Request);
	((FCurlHttpRequest*)Request)->OnComplete(OptionalReturnCode ? *OptionalReturnCode : CURLE_OK);
}

void FCurlHttpConnectionPool::CurlLock(CURL* Curl, curl_lock_data Data, curl_lock_access Access, void* Param)
{
	FCurlHttpConnectionPool& ConnectionPool = *(FCurlHttpConnectionPool*)Param;
	if (Access == CURL_LOCK_ACCESS_SHARED)
	{
		ConnectionPool.Locks[Data].ReadLock();
	}
	else
	{
		ConnectionPool.Locks[Data].WriteLock();
		ConnectionPool.WriteLocked[Data] = true;
	}
}

void FCurlHttpConnectionPool::CurlUnlock(CURL* Curl, curl_lock_data Data, void* Param)
{
	FCurlHttpConnectionPool& ConnectionPool = *(FCurlHttpConnectionPool*)Param;
	if (!ConnectionPool.WriteLocked[Data])
	{
		ConnectionPool.Locks[Data].ReadUnlock();
	}
	else
	{
		ConnectionPool.WriteLocked[Data] = false;
		ConnectionPool.Locks[Data].WriteUnlock();
	}
}

inline void FCurlHttpConnectionPool::AssertShareCodeOk(const CURLSHcode Code)
{
	checkf(Code == CURLSHE_OK, TEXT("Error in curl_share operation: %hs"), curl_share_strerror(Code));
}

inline void FCurlHttpConnectionPool::AssertMultiCodeOk(const CURLMcode Code)
{
	checkf(Code == CURLM_OK, TEXT("Error in curl_multi operation: %hs"), curl_multi_strerror(Code));
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCurlHttpClient::FCurlHttpClient(FCurlHttpConnectionPool& InConnectionPool, const FHttpClientParams& InParams)
	: ConnectionPool(InConnectionPool)
	, Params(InParams)
{
	checkf(!Params.MaxRequests || !Params.MinRequests || Params.MinRequests <= Params.MaxRequests,
		TEXT("MinRequests (%u) exceeds MaxRequests (%u)"), Params.MinRequests, Params.MaxRequests);
	Params.MaxRequests = Params.MaxRequests ? Params.MaxRequests : FMath::Max(Params.MinRequests, 256u);
	Params.MinRequests = Params.MinRequests ? Params.MinRequests : FMath::Min(Params.MaxRequests, 16u);
}

FCurlHttpClient::~FCurlHttpClient()
{
	while (FCurlHttpRequest* Request = RequestPool.Pop())
	{
		delete Request;
		RequestCount.fetch_sub(1, std::memory_order_relaxed);
		RequestPoolCount.fetch_sub(1, std::memory_order_relaxed);
	}

	checkf(RequestCount.load(std::memory_order_relaxed) == 0,
		TEXT("Client has %u requests that were not destroyed by IHttpRequest::Destroy()."), RequestCount.load());

	ConnectionPool.DeleteClient(this);
}

THttpUniquePtr<IHttpRequest> FCurlHttpClient::TryCreateRequest(const FHttpRequestParams& RequestParams)
{
	if (IHttpRequest* Request = RequestPool.Pop())
	{
		RequestPoolCount.fetch_sub(1, std::memory_order_relaxed);
		return THttpUniquePtr<IHttpRequest>(Request);
	}

	if (RequestParams.bIgnoreMaxRequests)
	{
		RequestCount.fetch_add(1, std::memory_order_relaxed);
	}
	else if (!CurlHttp::Private::TryIncrement(RequestCount, Params.MaxRequests))
	{
		return nullptr;
	}

	return THttpUniquePtr<IHttpRequest>(new FCurlHttpRequest(*this));
}

void FCurlHttpClient::DeleteRequest(FCurlHttpRequest* Request)
{
	if (CurlHttp::Private::TryIncrement(RequestPoolCount, Params.MinRequests))
	{
		Request->Reset();
		RequestPool.Push(Request);
	}
	else
	{
		delete Request;
		RequestCount.fetch_sub(1, std::memory_order_relaxed);
	}

	if (Params.OnDestroyRequest)
	{
		Params.OnDestroyRequest();
	}
}

void FCurlHttpClient::SetDefaultOptions(CURL* Curl, FCurlHttpHeaders& Headers) const
{
	ConnectionPool.SetDefaultOptions(Curl, Headers);

	curl_easy_setopt(Curl, CURLOPT_HTTP_VERSION, ConvertVersion(Params.Version));

	curl_easy_setopt(Curl, CURLOPT_DNS_CACHE_TIMEOUT, (long)FMath::Min(Params.DnsCacheTimeout, (uint32)MAX_int32));
	curl_easy_setopt(Curl, CURLOPT_CONNECTTIMEOUT_MS, long(Params.ConnectTimeoutMs));
	curl_easy_setopt(Curl, CURLOPT_TIMEOUT_MS, long(Params.TotalTimeoutMs));
	curl_easy_setopt(Curl, CURLOPT_LOW_SPEED_LIMIT, long(Params.LowSpeedLimit));
	curl_easy_setopt(Curl, CURLOPT_LOW_SPEED_TIME, long(Params.LowSpeedTime));
	curl_easy_setopt(Curl, CURLOPT_USE_SSL, ConvertTlsLevel(Params.TlsLevel));
	curl_easy_setopt(Curl, CURLOPT_FOLLOWLOCATION, long(Params.bFollowRedirects));
	curl_easy_setopt(Curl, CURLOPT_POSTREDIR, long(
		(Params.bFollow301Post ? CURL_REDIR_POST_301 : 0) |
		(Params.bFollow302Post ? CURL_REDIR_POST_302 : 0) |
		(Params.bFollow303Post ? CURL_REDIR_POST_303 : 0)));
	if (Params.bBypassProxy)
	{
		curl_easy_setopt(Curl, CURLOPT_NOPROXY, "*");
	}
	curl_easy_setopt(Curl, CURLOPT_VERBOSE, long(Params.bVerbose));
}

long FCurlHttpClient::ConvertVersion(const EHttpVersion Version)
{
	switch (Version)
	{
	case EHttpVersion::None:   return CURL_HTTP_VERSION_NONE;
	case EHttpVersion::V1_0:   return CURL_HTTP_VERSION_1_0;
	case EHttpVersion::V1_1:   return CURL_HTTP_VERSION_1_1;
	case EHttpVersion::V2:     return CURL_HTTP_VERSION_2_0;
	case EHttpVersion::V2Only: return CURL_HTTP_VERSION_2_PRIOR_KNOWLEDGE;
	default: checkNoEntry();   return CURL_HTTP_VERSION_NONE;
	}
}

long FCurlHttpClient::ConvertTlsLevel(const EHttpTlsLevel Level)
{
	switch (Level)
	{
	case EHttpTlsLevel::None: return CURLUSESSL_NONE;
	case EHttpTlsLevel::Try:  return CURLUSESSL_TRY;
	case EHttpTlsLevel::All:  return CURLUSESSL_ALL;
	default: checkNoEntry();  return CURLUSESSL_NONE;
	}
}

inline bool FCurlHttpClient::BeginAsyncRequest(FCurlHttpResponse* Response)
{
	Response->SetClient(this);
	return ConnectionPool.BeginAsyncRequest(Response);
}

inline void FCurlHttpClient::CancelAsyncRequest(FCurlHttpResponse* Response)
{
	ConnectionPool.CancelAsyncRequest(Response);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCurlHttpRequest::FCurlHttpRequest(FCurlHttpClient& InClient)
	: Client(InClient)
{
	CURLcode InitResult = (CURLcode)UE::Curl::ConditionalInitialize(UE::Curl::InitializeData);
	if (InitResult != CURLE_OK)
	{
		UE_LOGF(LogHttp, Warning, "FCurlHttpRequest could not initialize libcurl (result=%d).", (int32)InitResult);
	}
	Curl = curl_easy_init();
	SetDefaultOptions();
}

FCurlHttpRequest::~FCurlHttpRequest()
{
	//CheckIdle(TEXT("~FCurlHttpRequest"));
	if (Response.load(std::memory_order_relaxed))
	{
		UE_LOGFMT(LogHttp, Warning, "{Method} {Uri}: {FunctionName} requires the request to be idle.",
			LexToString(Method), Uri, TEXT("~FCurlHttpRequest"));
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
	}
	curl_easy_cleanup(Curl);
}

void FCurlHttpRequest::Reset()
{
	CheckIdle(TEXT("Reset"));
	curl_easy_reset(Curl);
	Headers.Reset();
	Uri.Reset();
	UnixSocketPath.Empty();
	Body.Reset();
	SetDefaultOptions();
}

void FCurlHttpRequest::SetDefaultOptions()
{
	Method = EHttpMethod::Get;

	Client.SetDefaultOptions(Curl, Headers);

	curl_easy_setopt(Curl, CURLOPT_PRIVATE, this);
	curl_easy_setopt(Curl, CURLOPT_READDATA, this);
	curl_easy_setopt(Curl, CURLOPT_READFUNCTION, CurlRead);
	curl_easy_setopt(Curl, CURLOPT_SEEKDATA, this);
	curl_easy_setopt(Curl, CURLOPT_SEEKFUNCTION, CurlSeek);
#if WITH_SSL
	curl_easy_setopt(Curl, CURLOPT_SSL_CTX_DATA, this);
	curl_easy_setopt(Curl, CURLOPT_SSL_CTX_FUNCTION, CurlSslContext);
#endif
}

void FCurlHttpRequest::CheckIdle(const TCHAR* FunctionName)
{
	checkf(!Response.load(std::memory_order_relaxed),
		TEXT("%.*hs %hs: %s requires the request to be idle."),
		LexToString(Method).Len(), LexToString(Method).GetData(), *Uri, FunctionName);
}

void FCurlHttpRequest::SetUri(const FAnsiStringView InUri)
{
	CheckIdle(TEXT("SetUri"));
	Uri = InUri;
	curl_easy_setopt(Curl, CURLOPT_URL, *Uri);
}

void FCurlHttpRequest::SetUnixSocketPath(const FAnsiStringView InSocketPath)
{
	CheckIdle(TEXT("SetUnixSocketPath"));
	UnixSocketPath.Empty();
	UnixSocketPath.Append(InSocketPath);

	if (UnixSocketPath.Len() > 0)
	{
		curl_easy_setopt(Curl, CURLOPT_UNIX_SOCKET_PATH, *UnixSocketPath);
	}
	else
	{
		curl_easy_setopt(Curl, CURLOPT_UNIX_SOCKET_PATH, NULL);
	}
}

void FCurlHttpRequest::SetMethod(const EHttpMethod InMethod)
{
	CheckIdle(TEXT("SetMethod"));
	Method = InMethod;
	switch (Method)
	{
	case EHttpMethod::Get:
		curl_easy_setopt(Curl, CURLOPT_HTTPGET, 1L);
		break;
	case EHttpMethod::Put:
		curl_easy_setopt(Curl, CURLOPT_UPLOAD, 1L);
		break;
	case EHttpMethod::Post:
		curl_easy_setopt(Curl, CURLOPT_POST, 1L);
		break;
	case EHttpMethod::Head:
		curl_easy_setopt(Curl, CURLOPT_NOBODY, 1L);
		break;
	case EHttpMethod::Delete:
		curl_easy_setopt(Curl, CURLOPT_CUSTOMREQUEST, "DELETE");
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FCurlHttpRequest::SetBody(const FCompositeBuffer& InBody)
{
	CheckIdle(TEXT("SetBody"));
	Body = InBody;
}

void FCurlHttpRequest::AddHeader(const FAnsiStringView Name, const FAnsiStringView Value)
{
	CheckIdle(TEXT("AddHeader"));
	Headers.AddHeader(*WriteToAnsiString<256>(Name, ANSITEXTVIEW(": "), Value));
}

void FCurlHttpRequest::Send(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse)
{
	CheckIdle(TEXT("Send"));
	if (CreateResponse(Receiver, OutResponse))
	{
		OnComplete(curl_easy_perform(Curl));
	}
}

void FCurlHttpRequest::SendAsync(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse)
{
	CheckIdle(TEXT("SendAsync"));
	if (FCurlHttpResponse* LocalResponse = CreateResponse(Receiver, OutResponse))
	{
		if (!Client.BeginAsyncRequest(LocalResponse))
		{
			OnComplete(curl_easy_perform(Curl));
		}
	}
}

FCurlHttpResponse* FCurlHttpRequest::CreateResponse(IHttpReceiver* Receiver, THttpUniquePtr<IHttpResponse>& OutResponse)
{
	BodyOffset = 0;

	if (const uint64 BodySize = Body.GetSize(); BodySize > 0)
	{
		switch (Method)
		{
		case EHttpMethod::Put:
			curl_easy_setopt(Curl, CURLOPT_INFILESIZE_LARGE, curl_off_t(BodySize));
			break;
		case EHttpMethod::Post:
			curl_easy_setopt(Curl, CURLOPT_POSTFIELDSIZE_LARGE, curl_off_t(BodySize));
			break;
		default:
			checkf(false, TEXT("%s"), *WriteToString<256>(TEXTVIEW("Method "), LexToString(Method),
				TEXTVIEW(" must not include a body but a body of "), BodySize, TEXTVIEW(" bytes was provided.")));
			break;
		}
	}

	curl_easy_setopt(Curl, CURLOPT_HTTPHEADER, Headers.GetList());

	FCurlHttpResponse* LocalResponse = new FCurlHttpResponse(Curl, Method, Uri, Receiver);
	OutResponse = THttpUniquePtr<IHttpResponse>(LocalResponse);

	if (LocalResponse->Create())
	{
		verify(!Response.exchange(LocalResponse, std::memory_order_release));
		return LocalResponse;
	}
	LocalResponse->SetComplete(CURLE_OK);
	return nullptr;
}

void FCurlHttpRequest::OnComplete(CURLcode Code)
{
	if (FCurlHttpResponse* LocalResponse = Response.exchange(nullptr, std::memory_order_acquire))
	{
		LocalResponse->SetComplete(Code);
	}
}

size_t FCurlHttpRequest::CurlRead(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param)
{
	FCurlHttpRequest* Request = (FCurlHttpRequest*)Param;
	const size_t TotalSize = Request->Body.GetSize();
	const size_t Offset = Request->BodyOffset;
	const size_t Size = FMath::Min(TotalSize - Offset, SizeInBlocks * BlockSizeInBytes);
	Request->Body.CopyTo(MakeMemoryView(Ptr, Size), Offset);
	Request->BodyOffset += Size;
	return Size;
}

size_t FCurlHttpRequest::CurlSeek(void* Param, curl_off_t Offset, int Origin)
{
	FCurlHttpRequest* Request = (FCurlHttpRequest*)Param;

	const size_t TotalSize = Request->Body.GetSize();
	size_t AbsoluteOffset = 0;

	switch (Origin)
	{
	case SEEK_SET: AbsoluteOffset = Offset; break;
	case SEEK_CUR: AbsoluteOffset = Request->BodyOffset + Offset; break;
	case SEEK_END: AbsoluteOffset = TotalSize + Offset; break;
	}

	if (AbsoluteOffset > TotalSize)
	{
		return CURL_SEEKFUNC_FAIL;
	}

	Request->BodyOffset = AbsoluteOffset;
	return CURL_SEEKFUNC_OK;
}

#if WITH_SSL
CURLcode FCurlHttpRequest::CurlSslContext(CURL* Curl, void* Context, void* Param)
{
	SSL_CTX* SslContext = (SSL_CTX*)Context;
	FSslModule::Get().GetCertificateManager().AddCertificatesToSslContext(SslContext);
	SSL_CTX_set_verify(SslContext, SSL_CTX_get_verify_mode(SslContext), SslCertVerify);
	SSL_CTX_set_app_data(SslContext, Param);
	return CURLE_OK;
}

int FCurlHttpRequest::SslCertVerify(int PreverifyOk, X509_STORE_CTX* Context)
{
	if (PreverifyOk == 1)
	{
		SSL* Ssl = (SSL*)X509_STORE_CTX_get_ex_data(Context, SSL_get_ex_data_X509_STORE_CTX_idx());
		check(Ssl);

		SSL_CTX* SslContext = SSL_get_SSL_CTX(Ssl);
		check(SslContext);

		FCurlHttpRequest* Request = (FCurlHttpRequest*)SSL_CTX_get_app_data(SslContext);
		check(Request);

		// Extract the domain from the URI.
		FAnsiStringView Domain = Request->Uri;
		if (const int32 SchemeIndex = String::FindFirst(Domain, ANSITEXTVIEW("://")); SchemeIndex != INDEX_NONE)
		{
			Domain.RightChopInline(SchemeIndex + ANSITEXTVIEW("://").Len());
		}
		if (const int32 SlashIndex = String::FindFirstChar(Domain, '/'); SlashIndex != INDEX_NONE)
		{
			Domain.LeftInline(SlashIndex);
		}
		if (const int32 AtIndex = String::FindFirstChar(Domain, '@'); AtIndex != INDEX_NONE)
		{
			Domain.RightChopInline(AtIndex + 1);
		}
		const auto RemovePort = [](FAnsiStringView& Authority)
		{
			if (const int32 ColonIndex = String::FindLastChar(Authority, ':'); ColonIndex != INDEX_NONE)
			{
				Authority.LeftInline(ColonIndex);
			}
		};
		if (Domain.StartsWith('['))
		{
			if (const int32 LastBracketIndex = String::FindLastChar(Domain, ']'); LastBracketIndex != INDEX_NONE)
			{
				Domain.MidInline(1, LastBracketIndex - 1);
			}
			else
			{
				RemovePort(Domain);
			}
		}
		else
		{
			RemovePort(Domain);
		}

		if (!FSslModule::Get().GetCertificateManager().VerifySslCertificates(Context, FString(Domain)))
		{
			PreverifyOk = 0;
		}
	}

	return PreverifyOk;
}
#endif

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FCurlHttpResponse::FCurlHttpResponse(CURL* InCurl, EHttpMethod InMethod, FAnsiStringView InUri, IHttpReceiver* InReceiver)
	: Method(InMethod)
	, Curl(InCurl)
	, Receiver(InReceiver)
{
	CaptureInheritedContext();

	// Release() is called by Destroy().
	AddRef();

	Uri = InUri;
	checkf(Receiver, TEXT("Receiver must not be null for %s"), *WriteToString<256>(*this));

	curl_easy_setopt(Curl, CURLOPT_ERRORBUFFER, ErrorBuffer);
	curl_easy_setopt(Curl, CURLOPT_DEBUGDATA, this);
	curl_easy_setopt(Curl, CURLOPT_DEBUGFUNCTION, CurlDebug);
	curl_easy_setopt(Curl, CURLOPT_HEADERDATA, this);
	curl_easy_setopt(Curl, CURLOPT_HEADERFUNCTION, CurlHeader);
	curl_easy_setopt(Curl, CURLOPT_WRITEDATA, this);
	curl_easy_setopt(Curl, CURLOPT_WRITEFUNCTION, CurlWrite);
}

bool FCurlHttpResponse::Create()
{
	for (TGuardValue GuardReceiverFunction(ReceiverFunction, EHttpReceiverFunction::OnCreate);;)
	{
		IHttpReceiver* NewReceiver = Receiver->OnCreate(*this);
		if (Receiver == NewReceiver)
		{
			break;
		}
		checkf(NewReceiver, TEXT("Receiver must not be null for %s"), *WriteToString<256>(*this));
		Receiver = NewReceiver;
	}
	return !IsCanceled();
}

FCurlHttpResponse::~FCurlHttpResponse()
{
	checkf(!Curl, TEXT("Response must be completed before it is destroyed for %s"), *WriteToString<256>(*this));
}

void FCurlHttpResponse::Destroy()
{
	if (Curl)
	{
		Cancel();
	}

	// AddRef() is called by the constructor.
	Release();
}

void FCurlHttpResponse::ConditionallySetResponseStatus()
{
	if (StatusCode < 0)
	{
		long ResponseCode;
		curl_easy_getinfo(Curl, CURLINFO_RESPONSE_CODE, &ResponseCode);
		if (ResponseCode > 0)
		{
			StatusCode = int32(ResponseCode);
		}
	}
}

bool FCurlHttpResponse::WriteHeader(FAnsiStringView Header)
{
	if (bHasBody)
	{
		return !EnumHasAnyFlags(State.load(std::memory_order_relaxed), ECurlHttpResponseState::Canceled);
	}

	FInheritedContextScope InheritedContextScope = RestoreInheritedContext();

	// Reset headers between responses and keep only the last.
	if (!HeaderViews.IsEmpty())
	{
		HeaderLengths.Reset();
		HeaderViews.Reset();
		Headers.Reset();
		StatusCode = -1;
	}

	ConditionallySetResponseStatus();

	if (Header == ANSITEXTVIEW("\r\n"))
	{
		const ANSICHAR* Data = Headers.GetData();
		for (const int32 Len : HeaderLengths)
		{
			HeaderViews.Add(MakeStringView(Data, Len - 2));
			Data += Len;
		}
		for (TGuardValue GuardReceiverFunction(ReceiverFunction, EHttpReceiverFunction::OnHeaders);;)
		{
			IHttpReceiver* NewReceiver = Receiver->OnHeaders(*this);
			if (Receiver == NewReceiver)
			{
				break;
			}
			checkf(NewReceiver, TEXT("Receiver must not be null for %s"), *WriteToString<256>(*this));
			Receiver = NewReceiver;
		}
	}
	else
	{
		HeaderLengths.Add(Header.Len());
		Headers.Append(Header);
	}

	return !IsCanceled();
}

bool FCurlHttpResponse::WriteBody(FMemoryView Body)
{
	FInheritedContextScope InheritedContextScope = RestoreInheritedContext();

	ConditionallySetResponseStatus();

	bHasBody = true;

	while (!Body.IsEmpty() && !IsCanceled())
	{
		const FMemoryView InitialBody = Body;
		IHttpReceiver* NewReceiver;
		{
			TGuardValue GuardReceiverFunction(ReceiverFunction, EHttpReceiverFunction::OnBody);
			NewReceiver = Receiver->OnBody(*this, Body);
		}
		if (Receiver == NewReceiver)
		{
			checkf(Body.IsEmpty() || Body == InitialBody,
				TEXT("Receiver must consume the entire body for %s"), *WriteToString<256>(*this));
			break;
		}
		checkf(Body.IsEmpty() || Body.GetDataEnd() == InitialBody.GetDataEnd(),
			TEXT("Receiver must consume the body sequentially for %s"), *WriteToString<256>(*this));
		checkf(NewReceiver, TEXT("Receiver must not be null for %s"), *WriteToString<256>(*this));
		Receiver = NewReceiver;
	}

	return !IsCanceled();
}

void FCurlHttpResponse::SetComplete(CURLcode Code)
{
	Error = ErrorBuffer;
	if (IsCanceled())
	{
		ErrorCode = EHttpErrorCode::Canceled;
		Error = ANSITEXTVIEW("Canceled using IHttpResponse::Cancel()");
		StatusCode = 0;
	}
	else if (Code == CURLE_OK)
	{
		ErrorCode = EHttpErrorCode::None;
		ConditionallySetResponseStatus();
	}
	else
	{
		ErrorCode = ConvertErrorCode(Code);
		if (Error.IsEmpty())
		{
			Error = curl_easy_strerror(Code);
		}
		StatusCode = 0;
	}

	const auto GetSizeInfo = [this](CURLINFO Info)
	{
		curl_off_t Value;
		curl_easy_getinfo(Curl, Info, &Value);
		return uint64(Value);
	};
	const auto GetDoubleInfo = [this](CURLINFO Info)
	{
		double Value;
		curl_easy_getinfo(Curl, Info, &Value);
		return Value;
	};
	Stats.SendSize = GetSizeInfo(CURLINFO_SIZE_UPLOAD_T);
	Stats.RecvSize = GetSizeInfo(CURLINFO_SIZE_DOWNLOAD_T);
	Stats.SendRate = GetSizeInfo(CURLINFO_SPEED_UPLOAD_T);
	Stats.RecvRate = GetSizeInfo(CURLINFO_SPEED_DOWNLOAD_T);
	Stats.NameResolveTime = GetDoubleInfo(CURLINFO_NAMELOOKUP_TIME);
	Stats.ConnectTime = GetDoubleInfo(CURLINFO_CONNECT_TIME);
	Stats.TlsConnectTime = GetDoubleInfo(CURLINFO_APPCONNECT_TIME);
	Stats.PreTransferTime = GetDoubleInfo(CURLINFO_PRETRANSFER_TIME);
	Stats.StartTransferTime = GetDoubleInfo(CURLINFO_STARTTRANSFER_TIME);
	Stats.TotalTime = GetDoubleInfo(CURLINFO_TOTAL_TIME);

	Curl = nullptr;
	Client = nullptr;

	// Hold a reference to safely access Receiver, State, CompleteEvent, and FInheritedContextBase.
	TRefCountPtr<FCurlHttpResponse> Self(this);
	FInheritedContextScope InheritedContextScope = RestoreInheritedContext();

	for (TGuardValue GuardReceiverFunction(ReceiverFunction, EHttpReceiverFunction::OnComplete);;)
	{
		IHttpReceiver* NewReceiver = Receiver->OnComplete(*this);
		if (Receiver == NewReceiver || !NewReceiver)
		{
			break;
		}
		Receiver = NewReceiver;
	}

	CurlHttp::Private::AtomicFetchOr(State, ECurlHttpResponseState::Complete);
	CompleteEvent.Notify();

	if (UE_LOG_ACTIVE(LogHttp, Verbose))
	{
		TStringBuilder<80> StatsText;
		if (Stats.SendSize)
		{
			StatsText << TEXTVIEW("sent ") << Stats.SendSize << TEXTVIEW(" bytes, ");
		}
		if (Stats.RecvSize)
		{
			StatsText << TEXTVIEW("received ") << Stats.RecvSize << TEXTVIEW(" bytes, ");
		}
		StatsText.Appendf(TEXT("%.3f seconds"), Stats.TotalTime);
		UE_LOGF(LogHttp, Verbose, "%ls (%ls)", *WriteToString<256>(*this), *StatsText);
	}

	// DO NOT ACCESS THIS AGAIN PAST THIS POINT!
}

void FCurlHttpResponse::Cancel()
{
	const EHttpReceiverFunction LocalReceiverFunction = ReceiverFunction;
	checkf(LocalReceiverFunction != EHttpReceiverFunction::OnComplete,
		TEXT("Cancel() must not be called from within OnComplete for %s"), *WriteToString<256>(*this));

	// Cancel only once and only if not completed.
	const auto NotCompleteOrCanceled = [](ECurlHttpResponseState CheckState) -> bool
	{
		return !EnumHasAnyFlags(CheckState, ECurlHttpResponseState::Complete | ECurlHttpResponseState::Canceled);
	};
	if (CurlHttp::Private::AtomicFetchOrIf(State, ECurlHttpResponseState::Canceled, NotCompleteOrCanceled))
	{
		// Queue cancellation of an async request when called from outside of a receiver function.
		// Cancellation within a receiver is handled by WriterHeader or WriteBody returning false,
		// which causes an error in curl that causes the request to complete immediately.
		if (Client && LocalReceiverFunction == EHttpReceiverFunction::None)
		{
			Client->CancelAsyncRequest(this);
		}
	}

	// Wait for completion when called from outside of a receiver function.
	// Waiting within a receiver function would prevent the response from completing.
	if (LocalReceiverFunction == EHttpReceiverFunction::None)
	{
		CompleteEvent.Wait();
	}
}

int32 FCurlHttpResponse::GetStatusCode() const
{
	return ErrorCode == EHttpErrorCode::None ? StatusCode.load(std::memory_order_relaxed) : 0;
}

const FHttpResponseStats& FCurlHttpResponse::GetStats() const
{
	checkf(!Curl, TEXT("Stats accessed before they are available for %s"), *WriteToString<256>(*this));
	return Stats;
}

size_t FCurlHttpResponse::CurlDebug(CURL* Curl, curl_infotype DebugInfoType, char* DebugInfo, size_t DebugInfoSize, void* Param)
{
	if (DebugInfoType != CURLINFO_TEXT && DebugInfoType != CURLINFO_HEADER_IN)
	{
		return 0;
	}

	FAnsiStringView DebugText(DebugInfo, DebugInfoSize);
	DebugText.TrimStartAndEndInline();
	if (DebugText.IsEmpty())
	{
		return 0;
	}

	FCurlHttpResponse* Response = (FCurlHttpResponse*)Param;
	const FAnsiStringView Method = LexToString(Response->GetMethod());
	const FAnsiStringView Uri = Response->GetUri();
	switch (DebugInfoType)
	{
	case CURLINFO_TEXT:
		UE_LOGF(LogHttp, VeryVerbose, "%.*s %.*s (%p): [TEXT] %.*s",
			Method.Len(), Method.GetData(), Uri.Len(), Uri.GetData(), Response, DebugText.Len(), DebugText.GetData());
		break;
	case CURLINFO_HEADER_IN:
		UE_LOGF(LogHttp, VeryVerbose, "%.*s %.*s (%p): [RESPONSE] %.*s",
			Method.Len(), Method.GetData(), Uri.Len(), Uri.GetData(), Response, DebugText.Len(), DebugText.GetData());
		break;
	}
	return 0;
}

size_t FCurlHttpResponse::CurlHeader(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param)
{
	const int32 HeaderSize = IntCastChecked<int32>(SizeInBlocks * BlockSizeInBytes);
	return ((FCurlHttpResponse*)Param)->WriteHeader(MakeStringView((const ANSICHAR*)Ptr, HeaderSize)) ? HeaderSize : 0;
}

size_t FCurlHttpResponse::CurlWrite(void* Ptr, size_t SizeInBlocks, size_t BlockSizeInBytes, void* Param)
{
	const size_t Size = SizeInBlocks * BlockSizeInBytes;
	return ((FCurlHttpResponse*)Param)->WriteBody(MakeMemoryView(Ptr, Size)) ? Size : 0;
}

EHttpErrorCode FCurlHttpResponse::ConvertErrorCode(CURLcode Code)
{
	switch (Code)
	{
	case CURLE_OK:
		return EHttpErrorCode::None;
	case CURLE_COULDNT_RESOLVE_HOST:
		return EHttpErrorCode::ResolveHost;
	case CURLE_COULDNT_CONNECT:
		return EHttpErrorCode::Connect;
	case CURLE_SSL_CONNECT_ERROR:
		return EHttpErrorCode::TlsConnect;
	case CURLE_PEER_FAILED_VERIFICATION:
		return EHttpErrorCode::TlsPeerVerification;
	case CURLE_OPERATION_TIMEDOUT:
		return EHttpErrorCode::TimedOut;
	default:
		return EHttpErrorCode::Unknown;
	}
}

void FCurlHttpResponse::AddRef() const
{
	ReferenceCount.fetch_add(1, std::memory_order_relaxed);
}

void FCurlHttpResponse::Release() const
{
	if (ReferenceCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		delete this;
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

IHttpManager& IHttpManager::Get()
{
	static FCurlHttpManager CurlHttpManager;
	return CurlHttpManager;
}

} // UE
