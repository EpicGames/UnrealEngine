// Copyright Epic Games, Inc. All Rights Reserved.

#include "Http/HttpClient.h"

#include "Async/ManualResetEvent.h"
#include "Containers/LockFreeList.h"
#include "Memory/MemoryView.h"
#include "Misc/AsciiSet.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "String/Find.h"
#include "String/LexFromString.h"

namespace UE
{

FAnsiStringView LexToString(const EHttpMethod Method)
{
	switch (Method)
	{
	case EHttpMethod::Get:    return ANSITEXTVIEW("GET");
	case EHttpMethod::Put:    return ANSITEXTVIEW("PUT");
	case EHttpMethod::Post:   return ANSITEXTVIEW("POST");
	case EHttpMethod::Head:   return ANSITEXTVIEW("HEAD");
	case EHttpMethod::Delete: return ANSITEXTVIEW("DELETE");
	default:                  return ANSITEXTVIEW("UNKNOWN");
	}
}

bool TryLexFromString(EHttpMethod& OutMethod, const FAnsiStringView View)
{
	if (View == ANSITEXTVIEW("GET"))
	{
		OutMethod = EHttpMethod::Get;
	}
	else if (View == ANSITEXTVIEW("PUT"))
	{
		OutMethod = EHttpMethod::Put;
	}
	else if (View == ANSITEXTVIEW("POST"))
	{
		OutMethod = EHttpMethod::Post;
	}
	else if (View == ANSITEXTVIEW("HEAD"))
	{
		OutMethod = EHttpMethod::Head;
	}
	else if (View == ANSITEXTVIEW("DELETE"))
	{
		OutMethod = EHttpMethod::Delete;
	}
	else
	{
		return false;
	}
	return true;
}

FAnsiStringView LexToString(const EHttpMediaType MediaType)
{
	switch (MediaType)
	{
	case EHttpMediaType::Any:              return ANSITEXTVIEW("*/*");
	case EHttpMediaType::Binary:           return ANSITEXTVIEW("application/octet-stream");
	case EHttpMediaType::Text:             return ANSITEXTVIEW("text/plain");
	case EHttpMediaType::Json:             return ANSITEXTVIEW("application/json");
	case EHttpMediaType::Yaml:             return ANSITEXTVIEW("text/yaml");
	case EHttpMediaType::CbObject:         return ANSITEXTVIEW("application/x-ue-cb");
	case EHttpMediaType::CbPackage:        return ANSITEXTVIEW("application/x-ue-cbpkg");
	case EHttpMediaType::CbPackageOffer:   return ANSITEXTVIEW("application/x-ue-offer");
	case EHttpMediaType::CompressedBinary: return ANSITEXTVIEW("application/x-ue-comp");
	case EHttpMediaType::FormUrlEncoded:   return ANSITEXTVIEW("application/x-www-form-urlencoded");
	default:                               return ANSITEXTVIEW("unknown");
	}
}

bool TryLexFromString(EHttpMediaType& OutMediaType, const FAnsiStringView View)
{
	const int32 SlashIndex = String::FindFirstChar(View, '/');
	if (SlashIndex == INDEX_NONE)
	{
		return false;
	}

	const FAnsiStringView Type = View.Left(SlashIndex);
	const FAnsiStringView SubType = View.RightChop(SlashIndex + 1);
	if (Type == ANSITEXTVIEW("application"))
	{
		if (SubType == ANSITEXTVIEW("octet-stream"))
		{
			OutMediaType = EHttpMediaType::Binary;
		}
		else if (SubType == ANSITEXTVIEW("json"))
		{
			OutMediaType = EHttpMediaType::Json;
		}
		else if (SubType == ANSITEXTVIEW("x-ue-cb"))
		{
			OutMediaType = EHttpMediaType::CbObject;
		}
		else if (SubType == ANSITEXTVIEW("x-ue-cbpkg"))
		{
			OutMediaType = EHttpMediaType::CbPackage;
		}
		else if (SubType == ANSITEXTVIEW("x-ue-offer"))
		{
			OutMediaType = EHttpMediaType::CbPackageOffer;
		}
		else if (SubType == ANSITEXTVIEW("x-ue-comp"))
		{
			OutMediaType = EHttpMediaType::CompressedBinary;
		}
		else if (SubType == ANSITEXTVIEW("x-www-form-urlencoded"))
		{
			OutMediaType = EHttpMediaType::FormUrlEncoded;
		}
		else
		{
			return false;
		}
	}
	else if (Type == ANSITEXTVIEW("text"))
	{
		if (SubType == ANSITEXTVIEW("plain"))
		{
			OutMediaType = EHttpMediaType::Text;
		}
		else if (SubType == ANSITEXTVIEW("yaml"))
		{
			OutMediaType = EHttpMediaType::Yaml;
		}
		else
		{
			return false;
		}
	}
	else if (Type == ANSITEXTVIEW("*") && SubType == ANSITEXTVIEW("*"))
	{
		OutMediaType = EHttpMediaType::Any;
	}
	else
	{
		return false;
	}

	return true;
}

FAnsiStringView LexToString(EHttpErrorCode ErrorCode)
{
	switch (ErrorCode)
	{
	case EHttpErrorCode::None:                return ANSITEXTVIEW("None");
	case EHttpErrorCode::Unknown:             return ANSITEXTVIEW("Unknown");
	case EHttpErrorCode::Canceled:            return ANSITEXTVIEW("Canceled");
	case EHttpErrorCode::ResolveHost:         return ANSITEXTVIEW("ResolveHost");
	case EHttpErrorCode::Connect:             return ANSITEXTVIEW("Connect");
	case EHttpErrorCode::TlsConnect:          return ANSITEXTVIEW("TlsConnect");
	case EHttpErrorCode::TlsPeerVerification: return ANSITEXTVIEW("TlsPeerVerification");
	case EHttpErrorCode::TimedOut:            return ANSITEXTVIEW("TimedOut");
	default:                                  return ANSITEXTVIEW("Invalid");
	}

}

bool TryLexFromString(EHttpErrorCode& OutErrorCode, FAnsiStringView View)
{
	if (View == ANSITEXTVIEW("None"))
	{
		OutErrorCode = EHttpErrorCode::None;
	}
	else if (View == ANSITEXTVIEW("Unknown"))
	{
		OutErrorCode = EHttpErrorCode::Unknown;
	}
	else if (View == ANSITEXTVIEW("Canceled"))
	{
		OutErrorCode = EHttpErrorCode::Canceled;
	}
	else if (View == ANSITEXTVIEW("ResolveHost"))
	{
		OutErrorCode = EHttpErrorCode::ResolveHost;
	}
	else if (View == ANSITEXTVIEW("Connect"))
	{
		OutErrorCode = EHttpErrorCode::Connect;
	}
	else if (View == ANSITEXTVIEW("TlsConnect"))
	{
		OutErrorCode = EHttpErrorCode::TlsConnect;
	}
	else if (View == ANSITEXTVIEW("TlsPeerVerification"))
	{
		OutErrorCode = EHttpErrorCode::TlsPeerVerification;
	}
	else if (View == ANSITEXTVIEW("TimedOut"))
	{
		OutErrorCode = EHttpErrorCode::TimedOut;
	}
	else
	{
		return false;
	}
	return true;
}

static FAnsiStringView LexStatusCodeToString(int HttpCode)
{
	switch (HttpCode)
	{
		// 1xx Informational
	case 100: return "Continue";
	case 101: return "Switching Protocols";

		// 2xx Success
	case 200: return "OK";
	case 201: return "Created";
	case 202: return "Accepted";
	case 204: return "No Content";
	case 205: return "Reset Content";
	case 206: return "Partial Content";

		// 3xx Redirection
	case 300: return "Multiple Choices";
	case 301: return "Moved Permanently";
	case 302: return "Found";
	case 303: return "See Other";
	case 304: return "Not Modified";
	case 305: return "Use Proxy";
	case 306: return "Switch Proxy";
	case 307: return "Temporary Redirect";
	case 308: return "Permanent Redirect";

		// 4xx Client errors
	case 400: return "Bad Request";
	case 401: return "Unauthorized";
	case 402: return "Payment Required";
	case 403: return "Forbidden";
	case 404: return "Not Found";
	case 405: return "Method Not Allowed";
	case 406: return "Not Acceptable";
	case 407: return "Proxy Authentication Required";
	case 408: return "Request Timeout";
	case 409: return "Conflict";
	case 410: return "Gone";
	case 411: return "Length Required";
	case 412: return "Precondition Failed";
	case 413: return "Payload Too Large";
	case 414: return "URI Too Long";
	case 415: return "Unsupported Media Type";
	case 416: return "Range Not Satisfiable";
	case 417: return "Expectation Failed";
	case 418: return "I'm a teapot";
	case 421: return "Misdirected Request";
	case 422: return "Unprocessable Entity";
	case 423: return "Locked";
	case 424: return "Failed Dependency";
	case 425: return "Too Early";
	case 426: return "Upgrade Required";
	case 428: return "Precondition Required";
	case 429: return "Too Many Requests";
	case 431: return "Request Header Fields Too Large";

		// 5xx Server errors
	case 500: return "Internal Server Error";
	case 501: return "Not Implemented";
	case 502: return "Bad Gateway";
	case 503: return "Service Unavailable";
	case 504: return "Gateway Timeout";
	case 505: return "HTTP Version Not Supported";
	case 506: return "Variant Also Negotiates";
	case 507: return "Insufficient Storage";
	case 508: return "Loop Detected";
	case 510: return "Not Extended";
	case 511: return "Network Authentication Required";

	default: return "Unknown Result";
	}
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void IHttpRequest::SetContentType(const EHttpMediaType Type, const FAnsiStringView Param)
{
	TAnsiStringBuilder<64> Value;
	Value << LexToString(Type);
	if (!Param.IsEmpty())
	{
		Value << ANSITEXTVIEW("; ") << Param;
	}
	AddHeader(ANSITEXTVIEW("Content-Type"), Value);
}

void IHttpRequest::AddAcceptType(const EHttpMediaType Type, const float Weight)
{
	TAnsiStringBuilder<64> Value;
	Value << LexToString(Type);
	if (Weight != 1.0f)
	{
		Value.Appendf(";q=%.3f", Weight);
	}
	AddHeader(ANSITEXTVIEW("Accept"), Value);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FAnsiStringView IHttpResponse::GetHeader(const FAnsiStringView Name) const
{
	const int32 NameLen = Name.Len();
	for (const FAnsiStringView& Header : GetAllHeaders())
	{
		if (Header.StartsWith(Name, ESearchCase::IgnoreCase) &&
			Header.Len() > NameLen && Header.GetData()[NameLen] == ':')
		{
			return Header.RightChop(NameLen + 1).TrimStartAndEnd();
		}
	}
	return {};
}

int32 IHttpResponse::GetHeaders(FAnsiStringView Name, TArrayView<FAnsiStringView> OutValues) const
{
	int32 MatchIndex = 0;
	const int32 NameLen = Name.Len();
	const int32 ValuesCount = OutValues.Num();
	FAnsiStringView* Values = OutValues.GetData();
	for (const FAnsiStringView& Header : GetAllHeaders())
	{
		if (Header.StartsWith(Name, ESearchCase::IgnoreCase) &&
			Header.Len() > NameLen && Header.GetData()[NameLen] == ':')
		{
			if (ValuesCount > MatchIndex)
			{
				Values[MatchIndex] = Header.RightChop(NameLen + 1).TrimStartAndEnd();
			}
			++MatchIndex;
		}
	}
	return MatchIndex;
}

EHttpMediaType IHttpResponse::GetContentType() const
{
	const FAnsiStringView ContentType = GetHeader(ANSITEXTVIEW("Content-Type"));
	const FAnsiStringView ContentTypeNoParams = FAsciiSet::FindPrefixWithout(ContentType, "; \t");
	EHttpMediaType MediaType = EHttpMediaType::Any;
	TryLexFromString(MediaType, ContentTypeNoParams);
	return MediaType;
}

uint64 IHttpResponse::GetContentLength() const
{
	const FAnsiStringView ContentLength = GetHeader(ANSITEXTVIEW("Content-Length"));
	const FAnsiStringView ContentLengthNoParams = FAsciiSet::FindPrefixWithout(ContentLength, "; \t");
	if (ContentLengthNoParams.IsEmpty())
	{
		return -1;
	}
	uint64 ContentLengthValue = static_cast<uint64>(-1);
	LexFromString(ContentLengthValue, ContentLengthNoParams);
	return ContentLengthValue;
}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const IHttpResponse& Response)
{
	Builder << LexToString(Response.GetMethod()) << TEXT(' ') << Response.GetUri();

	if (const int32 StatusCode = Response.GetStatusCode(); StatusCode > 0)
	{
		Builder << TEXTVIEW(" -> ") << LexStatusCodeToString(StatusCode) << " (" << StatusCode << ")";
	}

	if (const FAnsiStringView Error = Response.GetError(); !Error.IsEmpty())
	{
		Builder << TEXTVIEW(": ") << Error;
	}

	return Builder;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FHttpByteArrayReceiver::FHttpByteArrayReceiver(TArray64<uint8>& OutArray, IHttpReceiver* InNext)
	: Array(OutArray)
	, Next(InNext)
{
	Array.Reset();
}

IHttpReceiver* FHttpByteArrayReceiver::OnBody(IHttpResponse& Response, FMemoryView& Data)
{
	if (Array.IsEmpty())
	{
		constexpr int64 MaxReserveSize = 96 * 1024 * 1024;
		if (const FAnsiStringView View = Response.GetHeader(ANSITEXTVIEW("Content-Length")); !View.IsEmpty())
		{
			constexpr int32 MaxStringLen = 16;
			ANSICHAR String[MaxStringLen];
			if (const int32 CopyLen = View.CopyString(String, MaxStringLen); CopyLen < MaxStringLen)
			{
				String[CopyLen] = '\0';
				const int64 ContentLength = FCStringAnsi::Atoi64(String);
				if (ContentLength > 0 && ContentLength <= MaxReserveSize)
				{
					Array.Reserve(ContentLength);
				}
			}
		}
	}
	Array.Append(static_cast<const uint8*>(Data.GetData()), int64(Data.GetSize()));
	return this;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct Http::Private::FHttpRequestQueueData
{
	struct FWaiter
	{
		THttpUniquePtr<IHttpRequest> Request;
		FManualResetEvent Event;
	};

	FHttpRequestQueueData(IHttpConnectionPool& ConnectionPool, const FHttpClientParams& ClientParams)
	{
		FHttpClientParams QueueParams = ClientParams;
		QueueParams.OnDestroyRequest = [this, OnDestroyRequest = MoveTemp(QueueParams.OnDestroyRequest)]
		{
			if (OnDestroyRequest)
			{
				OnDestroyRequest();
			}
			if (!Waiters.IsEmpty())
			{
				if (THttpUniquePtr<IHttpRequest> Request = Client->TryCreateRequest({}))
				{
					if (FWaiter* Waiter = Waiters.Pop())
					{
						Waiter->Request = MoveTemp(Request);
						Waiter->Event.Notify();
					}
				}
			}
		};
		Client = ConnectionPool.CreateClient(QueueParams);
	}

	THttpUniquePtr<IHttpRequest> CreateRequest(const FHttpRequestParams& Params)
	{
		if (Params.bIgnoreMaxRequests)
		{
			THttpUniquePtr<IHttpRequest> Request = Client->TryCreateRequest(Params);
			checkf(Request, TEXT("IHttpClient::TryCreateRequest returned null in spite of bIgnoreMaxRequests."));
			return Request;
		}

		while (THttpUniquePtr<IHttpRequest> Request = Client->TryCreateRequest(Params))
		{
			if (FWaiter* Waiter = Waiters.Pop())
			{
				Waiter->Request = MoveTemp(Request);
				Waiter->Event.Notify();
			}
			else
			{
				return Request;
			}
		}

		FWaiter LocalWaiter;
		Waiters.Push(&LocalWaiter);

		while (THttpUniquePtr<IHttpRequest> Request = Client->TryCreateRequest(Params))
		{
			if (FWaiter* Waiter = Waiters.Pop())
			{
				Waiter->Request = MoveTemp(Request);
				Waiter->Event.Notify();
			}
			if (LocalWaiter.Event.IsNotified())
			{
				checkf(LocalWaiter.Request, TEXT("CreateRequest returning null after IsNotified()."));
				return MoveTemp(LocalWaiter.Request);
			}
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(HttpRequestQueue_Wait);
		LocalWaiter.Event.Wait();
		checkf(LocalWaiter.Request, TEXT("CreateRequest returning null after Wait()."));
		return MoveTemp(LocalWaiter.Request);
	}

	THttpUniquePtr<IHttpClient> Client;
	TLockFreePointerListFIFO<FWaiter, 0> Waiters;
};

FHttpRequestQueue::FHttpRequestQueue(IHttpConnectionPool& ConnectionPool, const FHttpClientParams& ClientParams)
	: Data(MakePimpl<Http::Private::FHttpRequestQueueData>(ConnectionPool, ClientParams))
{
}

THttpUniquePtr<IHttpRequest> FHttpRequestQueue::CreateRequest(const FHttpRequestParams& Params)
{
	static_assert(sizeof(Params) == sizeof(bool), "CreateRequest only handles bIgnoreMaxRequests");
	check(Data);
	return Data->CreateRequest(Params);
}

} // UE
