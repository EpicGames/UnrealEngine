// Copyright Epic Games, Inc. All Rights Reserved.

#include "Http/HttpHostBuilder.h"

#include "Algo/MinElement.h"
#include "Async/ManualResetEvent.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "HAL/PlatformTime.h"
#include "Http/HttpClient.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Memory/MemoryFwd.h"
#include "Misc/ScopeExit.h"
#include "Serialization/CompactBinary.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

namespace UE
{

DEFINE_LOG_CATEGORY_STATIC(LogHttpHostBuilder, Display, All);

class FHttpBenchmarkReceiver final : public IHttpReceiver
{
public:
	FHttpBenchmarkReceiver(const FHttpBenchmarkReceiver&) = delete;
	FHttpBenchmarkReceiver& operator=(const FHttpBenchmarkReceiver&) = delete;

	using FOnComplete = TUniqueFunction<void(IHttpResponse& HttpResponse, FString& Host)>;

	explicit FHttpBenchmarkReceiver(const FStringView InHostEntry, FOnComplete&& InOperationComplete, IHttpReceiver* InNext = nullptr)
		: HostEntry(InHostEntry)
		, OperationComplete(MoveTemp(InOperationComplete))
		, Next(InNext)
	{
	}

private:
	IHttpReceiver* OnCreate(IHttpResponse& Response) final
	{
		return this;
	}

	IHttpReceiver* OnComplete(IHttpResponse& Response) final
	{
		OperationComplete(Response, HostEntry);

		return Next;
	}

private:
	FString HostEntry;
	FOnComplete OperationComplete;
	IHttpReceiver* Next;
};

class FHttpBenchmarkOperation final : public IHttpReceiver
{
public:
	FHttpBenchmarkOperation(const FHttpBenchmarkOperation&) = delete;
	FHttpBenchmarkOperation& operator=(const FHttpBenchmarkOperation&) = delete;

	using FOnComplete = TUniqueFunction<void(IHttpResponse& HttpResponse, FString& Host)>;

	explicit FHttpBenchmarkOperation(const FStringView InHostEntry, THttpUniquePtr<IHttpRequest>&& InRequest, FOnComplete&& InOperationComplete)
		: HostEntry(InHostEntry)
		, OperationComplete(MoveTemp(InOperationComplete))
		, Request(MoveTemp(InRequest))
		, Response(nullptr)
	{
		Request->SetUri(WriteToAnsiString<256>(HostEntry, ANSITEXTVIEW("/health/ready")));
	}

	void SendAsync()
	{
		Request->SendAsync(this, Response);
	}

	void Cancel()
	{
		Response->Cancel();
	}
	
private:
	IHttpReceiver* OnCreate(IHttpResponse& InResponse) final
	{
		return this;
	}

	IHttpReceiver* OnComplete(IHttpResponse& InResponse) final
	{
		OperationComplete(InResponse, HostEntry);

		return nullptr;
	}

private:
	FString HostEntry;
	FOnComplete OperationComplete;

	THttpUniquePtr<IHttpRequest> Request;
	THttpUniquePtr<IHttpResponse> Response;
};

void FHttpHostBuilder::AddFromEndpoint(const FAnsiStringView HostUrl, const FAnsiStringView AccessToken)
{
	FHttpConnectionPoolParams ConnectionPoolParams;
	const THttpUniquePtr<IHttpConnectionPool> LocalConnectionPool = IHttpManager::Get().CreateConnectionPool(ConnectionPoolParams);
	FHttpClientParams DefaultClientParams;
	// we want to keep this timeout fairly low in case the host is not reachable
	DefaultClientParams.ConnectTimeoutMs = 5000;
	THttpUniquePtr<IHttpClient> LocalClient = LocalConnectionPool->CreateClient(DefaultClientParams);

	FHttpRequestParams RequestParams;
	// setting ignore max requests guarantees that CreateRequest actually creates a request no matter how many requests are in flight
	RequestParams.bIgnoreMaxRequests = true;

	TArray64<uint8> BodyArray;
	FHttpByteArrayReceiver PeersReceiver(BodyArray);
	
	THttpUniquePtr<IHttpRequest> Request = LocalClient->TryCreateRequest(RequestParams);

	if (!AccessToken.IsEmpty())
	{
		Request->AddHeader(ANSITEXTVIEW("Authorization"), WriteToAnsiString<1024>(AccessToken));
	}
	Request->AddHeader(ANSITEXTVIEW("Accept"), ANSITEXTVIEW("application/x-ue-cb"));
	TAnsiStringBuilder<256> Uri(InPlace, HostUrl, ANSITEXTVIEW("/status/peers"));
	Request->SetUri(Uri);
	THttpUniquePtr<IHttpResponse> Response;
	Request->Send(&PeersReceiver, Response);

	if (Response->GetStatusCode() != 200)
	{
		UE_LOGFMT(LogHttpHostBuilder, Display, "Unsuccessful attempt to fetch hosts by endpoint from host: '{Host}'. Status code was: {Code}", Uri, Response->GetStatusCode());
		return;
	}

	FSharedBuffer SharedBuffer = MakeSharedBufferFromArray(MoveTemp(BodyArray));
	FCbObject PeersObject = FCbObject(SharedBuffer);

	for (FCbField Peer : PeersObject["peers"].AsArray())
	{
		FCbObject PeerObject = Peer.AsObject();
		for (FCbField Endpoint : PeerObject["endpoints"])
		{
			HostCandidates.AddUnique(FString(Endpoint.AsString()));
		}
	}
}

FString FHttpHostBuilder::GetHostCandidatesString() const
{
	return FString::Join(HostCandidates, TEXT(", "));
}

void FHttpHostBuilder::AddFromString(const FAnsiStringView HostList)
{
	String::ParseTokens(HostList, ';', [this](FAnsiStringView Host)
	{
		this->HostCandidates.Emplace(Host);
	}, String::EParseTokensOptions::SkipEmpty | String::EParseTokensOptions::Trim);
}

void FHttpHostBuilder::AddFromString(const FStringView HostList)
{
	String::ParseTokens(HostList, ';', [this](FStringView Host)
	{
		this->HostCandidates.Emplace(Host);
	}, String::EParseTokensOptions::SkipEmpty | String::EParseTokensOptions::Trim);
}

bool FHttpHostBuilder::ResolveHost(double WarningTimeoutSeconds, double TimeoutSeconds, FAnsiStringBuilderBase& OutHost, double& OutLatency)
{
	if (HostCandidates.Num() == 0)
	{
		// no hosts have been added
		return false;
	}

	if (HostCandidates.Num() == 1)
	{
		// if there is only one candidate we do not benchmark it and just return this straight away
		OutHost.Reset();
		OutHost.Append(HostCandidates[0]);

		OutLatency = 0.0;
		return true;
	}

	bool bHostFound = BenchmarkHostList(HostCandidates, WarningTimeoutSeconds, TimeoutSeconds, OutHost, OutLatency);

	// if no valid host was found, set the host to the first candidate, even if this will likely not work to connect to its still better then no options
	if (!bHostFound)
	{
		OutHost.Reset();
		OutHost.Append(HostCandidates[0]);
		OutLatency = 0.0;
	}
	return bHostFound;
}

bool FHttpHostBuilder::BenchmarkHostList(TConstArrayView<FString> InHostCandidates, double WarningTimeoutSeconds, double TimeoutSeconds, FAnsiStringBuilderBase& OutHost, double& OutLatency)
{
	const double StartTime = FPlatformTime::Seconds();

	const uint32 MaxTotalConnections = InHostCandidates.Num();
	FHttpConnectionPoolParams ConnectionPoolParams;
	ConnectionPoolParams.MaxConnections = MaxTotalConnections;
	ConnectionPoolParams.MinConnections = MaxTotalConnections;
	const THttpUniquePtr<IHttpConnectionPool> LocalConnectionPool = IHttpManager::Get().CreateConnectionPool(ConnectionPoolParams);
	const FHttpClientParams DefaultClientParams;
	THttpUniquePtr<IHttpClient> LocalClient = LocalConnectionPool->CreateClient(DefaultClientParams);

	struct FLatencySortableHost
	{
		double Latency;
		FString Host;
	};
	TArray<FLatencySortableHost> SortedHostList;
	TArray<TUniquePtr<FHttpBenchmarkOperation>> Operations;

	FMutex Mutex;
	FManualResetEvent HostBenchmarkDone;

	std::atomic<int32> FailedBenchmarkAttempts = 0;
	bool bAllHostsFailed = false;
	const int32 CountOfBenchmarksToRun = InHostCandidates.Num();
	for (const FString& HostCandidate : InHostCandidates)
	{
		// Benchmark each of the hosts
		FHttpRequestParams RequestParams;
		// setting ignore max requests guarantees that CreateRequest actually creates a request no matter how many requests are in flight
		RequestParams.bIgnoreMaxRequests = true;

		TUniquePtr<FHttpBenchmarkOperation> Operation = MakeUnique<FHttpBenchmarkOperation>(HostCandidate, LocalClient->TryCreateRequest(RequestParams), [&SortedHostList, &Mutex, &HostBenchmarkDone, &FailedBenchmarkAttempts, &bAllHostsFailed, &CountOfBenchmarksToRun](const IHttpResponse& Response, FString& Host)
		{
			if (Response.GetStatusCode() != 200)
			{
				if (FailedBenchmarkAttempts.fetch_add(1, std::memory_order_relaxed) + 1 == CountOfBenchmarksToRun)
				{
					bAllHostsFailed = true;
					HostBenchmarkDone.Notify();
				}

				return;
			}
			// grab stats and store in dictionary, needs to get the response for this
			const FHttpResponseStats& Stats = Response.GetStats();

			FLatencySortableHost HostLatencyRecord;
			HostLatencyRecord.Latency = Stats.GetLatency();
			HostLatencyRecord.Host = Host;

			{
				TUniqueLock Lock(Mutex);
				SortedHostList.Add(HostLatencyRecord);
			}

			HostBenchmarkDone.Notify();
		});

		Operation->SendAsync();
		Operations.Emplace(MoveTemp(Operation));
	}
	bool bValidHostFound = HostBenchmarkDone.WaitFor(FMonotonicTimeSpan::FromSeconds(WarningTimeoutSeconds));

	// all benchmarks failed
	if (bAllHostsFailed)
	{
		UE_LOGF(LogHttpHostBuilder, Display, "No valid host found as all benchmark attempts had errors");

		OutHost.Reset();
		OutHost.Append(InHostCandidates[0]);
		OutLatency = 0.0;

		return false;
	}

	{
		ON_SCOPE_EXIT {
			// cancel any outstanding benchmark request
			for (const TUniquePtr<FHttpBenchmarkOperation>& Operation : Operations)
			{
				Operation->Cancel();
			}
		};

		if (!bValidHostFound)
		{
			// warn that benchmarking is taking a lot of time
			UE_LOGF(LogHttpHostBuilder, Display, "HTTP Benchmarking is slow, continuing to wait to determine ideal host...");

			bValidHostFound = HostBenchmarkDone.WaitFor(FMonotonicTimeSpan::FromSeconds(TimeoutSeconds));
			if (!bValidHostFound)
			{
				UE_LOGF(LogHttpHostBuilder, Display, "No valid host found while benchmarking after timeout was reached");

				OutHost.Reset();
				OutHost.Append(InHostCandidates[0]);
				OutLatency = 0.0;
				
				return false;
			}
		}
		
		{
			TUniqueLock Lock(Mutex);

			FLatencySortableHost* MinElement = Algo::MinElementBy(SortedHostList, &FLatencySortableHost::Latency);

			if (MinElement != nullptr)
			{
				OutHost.Reset();
				OutHost.Append(MinElement->Host);
				OutLatency = MinElement->Latency;
			}
			else
			{
				// no hosts were found
				OutHost.Reset();
				OutHost.Append(InHostCandidates[0]);
				OutLatency = 0.0;

				UE_LOGF(LogHttpHostBuilder, Display, "Failed to determine fastest host option because we had no valid options");
			}
		}
	}

	const double EndTime = FPlatformTime::Seconds();
	const double BenchmarkingDuration = EndTime - StartTime;

	UE_LOGF(LogHttpHostBuilder, Display, "Resolved to using host '%s' based on HTTP benchmark with a estimated latency of '%.0fms'. Spent %.0fms doing benchmarking.", *OutHost, OutLatency * 1000, BenchmarkingDuration * 1000);
	return true;
}

} // UE